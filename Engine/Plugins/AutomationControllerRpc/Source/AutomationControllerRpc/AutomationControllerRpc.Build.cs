// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildTool;

public class AutomationControllerRpc : ModuleRules
{
	public AutomationControllerRpc(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"ExternalRpcRegistry",
				"Json",
				"HTTP",
				"HTTPServer",
				"SessionServices"
			}
			);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateDependencyModuleNames.Add("AutomationController");
		}
	}
}
