// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationControllerRpcModule.h"

#include "AutomationControllerRpcRegistrationComponent.h"
#include "Misc/CommandLine.h"

#define LOCTEXT_NAMESPACE "FAutomationControllerRpcModule"

DEFINE_LOG_CATEGORY_STATIC(LogAutomationControllerRpcModule, Log, All);

void FAutomationControllerRpcModule::StartupModule()
{
#if WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
	int32 RpcPort = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("rpcport="), RpcPort))
	{
		// Need to fetch the ExternalRpcRegistry to start the RPC server and register any callbacks if not already registered
		UExternalRpcRegistry* RpcRegistry = UExternalRpcRegistry::GetInstance();
		if (RpcRegistry == nullptr)
		{
			UE_LOGF(LogAutomationControllerRpcModule, Warning, "Unable to create RPC Registry Instance. This might lead to issues using the RPC Registry.");
		}
		else if (UAutomationControllerRpcRegistrationComponent* ObjectInstance = UAutomationControllerRpcRegistrationComponent::GetInstance())
		{
			AutomationControllerRpcRegistrationComponent = ObjectInstance;
		}
	}
#endif // WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
}

void FAutomationControllerRpcModule::ShutdownModule()
{
	AutomationControllerRpcRegistrationComponent = nullptr;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAutomationControllerRpcModule, AutomationControllerRpc)