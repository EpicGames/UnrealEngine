// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationControllerRpcRegistrationComponent.h"

#if WITH_AUTOMATION_TESTS

#include "AutomationControllerSettings.h"
#include "AutomationGroupFilter.h"
#include "Dom/JsonObject.h"
#include "IAutomationControllerModule.h"
#include "ISessionServicesModule.h"
#include "Misc/App.h"
#include "Misc/FilterCollection.h"
#include "Serialization/JsonSerializer.h"

#endif // WITH_AUTOMATION_TESTS

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationControllerRpcRegistrationComponent)

DEFINE_LOG_CATEGORY_STATIC(LogAutomationControllerRpcRegistrationComponent, Log, All);
DEFINE_LOG_CATEGORY_STATIC(LogAutomationControllerRpcBridge, Log, All);

#if WITH_AUTOMATION_TESTS

namespace {

/** Frequently used statuses supplied when generating a response. */
const FString StatusInitializing = TEXT("Initializing");
const FString StatusIdle = TEXT("Idle");
const FString StatusRunning = TEXT("Running");
const FString StatusSuccess = TEXT("Success");
const FString StatusError = TEXT("Error");

/**
 * Helper method to create a simple report with a status and a description.
 * 
 * @param Status		Status to be provided
 * @param Description	Information associated with the status
 * 
 * @return the serialized JSON string
 */
FString CreateSimpleJsonReport(const FString& Status, const FString& Description)
{
	FString Report;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&Report);

	TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject());
	RootObject->SetStringField(TEXT("Status"), Status);
	RootObject->SetStringField(TEXT("Description"), Description);
	FJsonSerializer::Serialize(RootObject.ToSharedRef(), JsonWriter);
	return Report;
}

} //anonymous

/** Private helper class to manage and maintain the AutomationController during the RPC flow */
class FAutomationControllerRpcBridge
{
public:
	FAutomationControllerRpcBridge() = default;

	~FAutomationControllerRpcBridge()
	{
		ResetInitializationHandles();

		if (AutomationController.IsValid() && TestReportGeneratedHandle.IsValid())
		{
			AutomationController->OnTestReportGenerated().Remove(TestReportGeneratedHandle);
		}
	}

	/**
	 * Initializes the AutomationController and all of its associated delegate handles while it sets up the workers and finds all registered tests.
	 * 
	 * @return the serialized JSON string with success or error information
	 */
	FString Init()
	{
		const FString InitializingDescription = TEXT("AutomationController is in the process of initializing. Get the AutomationControllerState to determine when tests are ready to be executed.");

		// Check if the automation controller does exist and if it's already in the process of initializing as we don't want to interrupt the previous preparations
		if (AutomationController != nullptr && IsPreparingToRun())
		{
			return CreateSimpleJsonReport(StatusInitializing, InitializingDescription);
		}

		// Change the state to uninitialized and proceed to initialize the AutomationController
		AutomationControllerState = EAutomationControllerState::Uninitialized;
		if (AutomationController == nullptr)
		{
			IAutomationControllerModule* AutomationControllerModule = FModuleManager::LoadModulePtr<IAutomationControllerModule>("AutomationController");
			if (AutomationControllerModule == nullptr)
			{
				return CreateSimpleJsonReport(StatusError, TEXT("Failed to load AutomationController module."));
			}

			AutomationController = AutomationControllerModule->GetAutomationController();
			if (AutomationController == nullptr)
			{
				return CreateSimpleJsonReport(StatusError, TEXT("Unable to fetch AutomationController from module. No valid AutomationController found."));
			}
		}

		// Get our default session Id from the App
		SessionID = GetSessionId();

		AutomationController->Init();
		FAutomationTestFramework::Get().LoadTestTagMappings();

		// Register for the callback that tells us there are tests available
		if (!TestsAvailableHandle.IsValid())
		{
			TestsAvailableHandle = AutomationController->OnTestsAvailable().AddRaw(this, &FAutomationControllerRpcBridge::HandleAvailableTestCallback);
		}

		// Register for the callback that tells us tests have been refreshed
		if (!TestsRefreshedHandle.IsValid())
		{
			TestsRefreshedHandle = AutomationController->OnTestsRefreshed().AddRaw(this, &FAutomationControllerRpcBridge::HandleRefreshTestCallback);
		}

		AutomationControllerState = EAutomationControllerState::FindingWorkers;
		AutomationController->RequestAvailableWorkers(SessionID);

		return CreateSimpleJsonReport(StatusInitializing, InitializingDescription);
	}

	/** Gets the current state of the AutomationController */
	EAutomationControllerState GetState()
	{
		return AutomationControllerState;
	}

	/**
	 * Fetches a list of available tests from the AutomationController.
	 *
	 * @param TestFilter	Test name filter applied to the list of all tests
	 * @param TagFilter		Tag filter applied to the list of all tests
	 * 
	 * @return the serialized JSON string with the list of available tests or error information
	 */
	FString GetAvailableTests(const FString& TestFilter, const FString& TagFilter)
	{
		if (AutomationController == nullptr)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("No valid AutomationController found."));
		}
		else if (AutomationControllerState == EAutomationControllerState::Uninitialized)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not been initialized. Unable to fetch the list of available tests."));
		}

		// Create a filter to add to the automation controller, otherwise we won't get visibility into our available tests
		TArray<FString> AllAvailableTests;
		TSharedPtr<AutomationFilterCollection> AutomationFilters = MakeShareable(new AutomationFilterCollection());

		// Apply any tag filtering
		if (!TagFilter.IsEmpty())
		{
			TSharedPtr<FAutomationGroupFilter> FilterTags = MakeShareable(new FAutomationGroupFilter());
			TArray<FAutomatedTestTagFilter> TagList;
			TagList.Add(FAutomatedTestTagFilter(TagFilter));
			FilterTags->SetTagFilter(TagList);
			AutomationFilters->Add(FilterTags);
		}

		AutomationController->SetFilter(AutomationFilters);
		AutomationController->SetVisibleTestsEnabled(true);

		if (!TestFilter.IsEmpty())
		{
			GenerateTestNames(TestFilter, AutomationFilters, AllAvailableTests);
		}
		else
		{
			AutomationController->GetEnabledTestNames(AllAvailableTests);
		}

		FString Report;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&Report);
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue("Status", StatusSuccess);
		JsonWriter->WriteValue("AvailableTests", AllAvailableTests);
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();

		return Report;
	}

	/**
	 * Runs the registered AutomationController tests.
	 *
	 * @param TestFilter	Test name filter applied to the list of all tests
	 * @param TagFilter		Tag filter applied to the list of all tests
	 * @param TestLoops		Number of times the tests should be executed
	 *
	 * @return the serialized JSON string with success or error information
	 */
	FString RunTests(const FString& TestFilter, const FString& TagFilter, int32 TestLoops)
	{
		if (AutomationController == nullptr)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("No valid AutomationController found."));
		}
		else if (AutomationControllerState == EAutomationControllerState::Uninitialized)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not been initialized. Unable to run tests."));
		}
		else if (IsPreparingToRun())
		{
			return CreateSimpleJsonReport(StatusInitializing, TEXT("AutomationController is in the process of initializing. Cannot run tests."));
		}

		// Clear delegate handles to avoid re-running tests due to multiple delegates being added or when refreshing session frontend
		// The handle will be registered in Init whenever a new command is executed
		ResetInitializationHandles();

		// Register the callback for when the tests generate a report
		if (!TestReportGeneratedHandle.IsValid())
		{
			TestReportGeneratedHandle = AutomationController->OnTestReportGenerated().AddRaw(this, &FAutomationControllerRpcBridge::HandleReportGeneratedTestCallback);
		}
		GeneratedTestReport = nullptr;

		AutomationController->StopTests();

		// Create a filter to add to the automation controller, otherwise we don't get any reports
		TArray<FString> FilteredTestNames;
		TSharedPtr<AutomationFilterCollection> AutomationFilters = MakeShareable(new AutomationFilterCollection());

		// Apply any tag filtering
		if (!TagFilter.IsEmpty())
		{
			TSharedPtr<FAutomationGroupFilter> FilterTags = MakeShareable(new FAutomationGroupFilter());
			TArray<FAutomatedTestTagFilter> TagList;
			TagList.Add(FAutomatedTestTagFilter(TagFilter));
			FilterTags->SetTagFilter(TagList);
			AutomationFilters->Add(FilterTags);
		}

		AutomationController->SetFilter(AutomationFilters);
		AutomationController->SetVisibleTestsEnabled(true);

		if (!TestFilter.IsEmpty())
		{
			GenerateTestNames(TestFilter, AutomationFilters, FilteredTestNames);
		}
		else
		{
			AutomationController->GetEnabledTestNames(FilteredTestNames);
		}

		int32 NumPasses = TestLoops > 0 ? TestLoops : 1;
		AutomationController->SetNumPasses(NumPasses);
		AutomationController->SetEnabledTests(FilteredTestNames);

		UE_LOGF(LogAutomationControllerRpcBridge, Display, "Preparing to execute %d tests.", FilteredTestNames.Num() * TestLoops);

		// Ticking is what will process the tests through the Automation controller
		if (!TickHandler.IsValid())
		{
			TickHandler = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FAutomationControllerRpcBridge::Tick));
		}

		AutomationController->RunTests();

		AutomationControllerState = EAutomationControllerState::DoingRequestedWork;

		return CreateSimpleJsonReport(StatusSuccess, TEXT("Tests have successfully been triggered. Get the AutomationControllerState to determine when tests have been completed."));
	}

	/** Generates a report from the previous test execution. */
	FString GenerateAutomationReportJson()
	{
		if (AutomationController == nullptr)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("No valid AutomationController found."));
		}
		else if (AutomationControllerState == EAutomationControllerState::Uninitialized)
		{
			return CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not been initialized. Unable to generate test report."));
		}
		else if (!GeneratedTestReport.IsValid())
		{
			return CreateSimpleJsonReport(StatusError, TEXT("No tests have been run or tests are still in progress. Unable to generate test report."));
		}

		FString OutputReport;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&OutputReport);

		TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject());
		RootObject->SetStringField(TEXT("Status"), StatusSuccess);
		RootObject->SetObjectField(TEXT("TestReport"), GeneratedTestReport);

		FJsonSerializer::Serialize(RootObject.ToSharedRef(), JsonWriter);
		return OutputReport;
	}

private:

	/** Checks to see if the AutomationController is ready to run tests. */
	bool IsPreparingToRun()
	{
		return (AutomationControllerState == EAutomationControllerState::FindingWorkers || AutomationControllerState == EAutomationControllerState::RequestTests);
	}

	/**	Gets a session identifier for this session. */
	FGuid GetSessionId()
	{
		// Set a default session identifier from our app instance
		FGuid SessionId = FApp::GetSessionId();

		ISessionServicesModule* SessionServicesModule = FModuleManager::LoadModulePtr<ISessionServicesModule>("SessionServices");
		if (SessionServicesModule == nullptr)
		{
			UE_LOGF(LogAutomationControllerRpcBridge, Display, "No Session Services available.");
			return SessionId;
		}

		TSharedPtr<ISessionManager> SessionManager = SessionServicesModule->GetSessionManager();
		if (SessionManager == nullptr)
		{
			UE_LOGF(LogAutomationControllerRpcBridge, Display, "No Session Manager available.");
			return SessionId;
		}

		// Bind the Session Id to the currently selected session
		if (TSharedPtr<ISessionInfo> ActiveSession = SessionManager->GetSelectedSession())
		{
			const bool bSessionIsValid = ActiveSession.IsValid() && (ActiveSession->GetSessionOwner() == FPlatformProcess::UserName(false));
			if (bSessionIsValid)
			{
				UE_LOGF(LogAutomationControllerRpcBridge, Display, "Set Session Identifier to '%ls' from the current active session.", *ActiveSession->GetSessionId().ToString());
				SessionId = ActiveSession->GetSessionId();
			}
		}
		else
		{
			// Select the first session available to bind to
			TArray<TSharedPtr<ISessionInfo>> AvailableSessions;
			SessionManager->GetSessions(AvailableSessions);
			if (AvailableSessions.Num() > 0)
			{
				UE_LOGF(LogAutomationControllerRpcBridge, Display, "Setting session to the first available.");
				SessionManager->SelectSession(AvailableSessions[0]);
				SessionId = AvailableSessions[0]->GetSessionId();
			}
		}

		return SessionId;
	}

	/** Resets all active delegate needed for test initialization handles. */
	void ResetInitializationHandles()
	{
		if (AutomationController != nullptr)
		{
			if (TestsAvailableHandle.IsValid())
			{
				AutomationController->OnTestsAvailable().Remove(TestsAvailableHandle);
				TestsAvailableHandle.Reset();
			}

			if (TestsRefreshedHandle.IsValid())
			{
				AutomationController->OnTestsRefreshed().Remove(TestsRefreshedHandle);
				TestsRefreshedHandle.Reset();
			}
		}

		if (TickHandler.IsValid())
		{
			FTSTicker::RemoveTicker(TickHandler);
			TickHandler.Reset();
		}
	}

	/** Delegate callback when the AutomationController has marked tests as being available. */
	void HandleAvailableTestCallback(EAutomationControllerModuleState::Type)
	{
		UE_LOGF(LogAutomationControllerRpcBridge, Log, "Tests have been marked as being available");
		AutomationController->RequestTests();
		AutomationControllerState = EAutomationControllerState::RequestTests;
	}

	/** Delegate callback when the AutomationController finished requesting test information. */
	void HandleRefreshTestCallback()
	{
		TArray<FString> FilteredTestNames;

		// This is called by the controller manager when it receives responses. We want to make sure it has a device, and we
		// want to make sure it's called while we're waiting for a response
		if (AutomationController->GetNumDeviceClusters() == 0 || AutomationControllerState != EAutomationControllerState::RequestTests)
		{
			UE_LOGF(LogAutomationControllerRpcBridge, Log, "Ignoring refresh from ControllerManager. NumDeviceClusters=%d, CurrentState=%ls", AutomationController->GetNumDeviceClusters(), *UEnum::GetValueAsString(AutomationControllerState));
			return;
		}

		// We have found some workers
		// Create a filter to add to the automation controller, otherwise we don't get any reports
		TSharedPtr<AutomationFilterCollection> AutomationFilters = MakeShareable(new AutomationFilterCollection());
		AutomationController->SetFilter(AutomationFilters);
		AutomationController->SetVisibleTestsEnabled(true);
		AutomationController->GetEnabledTestNames(FilteredTestNames);

		AutomationControllerState = EAutomationControllerState::Idle;
	}

	/** Delegate callback when the executed test has generated a report. */
	void HandleReportGeneratedTestCallback(const FString& TestReportJson)
	{
		// Reset our report generation handle as no other reports should be generated
		// Report will contain information on all of the specified passes that were triggered so there is no need to keep the handle to the delegate around
		if (TestReportGeneratedHandle.IsValid())
		{
			AutomationController->OnTestReportGenerated().Remove(TestReportGeneratedHandle);
			TestReportGeneratedHandle.Reset();
		}

		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(TestReportJson);
		if (!FJsonSerializer::Deserialize(JsonReader, GeneratedTestReport) || !GeneratedTestReport.IsValid())
		{
			UE_LOGF(LogAutomationControllerRpcBridge, Warning, "Failed to deserialize report:\n%ls", *TestReportJson);
		}
	}

	/** 
	 * Delegate callback when the Engine ticks.
	 * 
	 * @param DeltaTime	Time elapsed from the last tick
	 * 
	 * @return true if we need to continue to tick, false otherwise
	 */
	bool Tick(float DeltaTime)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FAutomationControllerRpcBridge_Tick);

		// Update the automation controller to keep it running
		AutomationController->Tick();

		// Update the automation process
		if (AutomationControllerState == EAutomationControllerState::DoingRequestedWork)
		{
			MonitorTests();
			return true;
		}
		else if (AutomationControllerState != EAutomationControllerState::Complete)
		{
			// We've entered here outside of an expected state as ticking should not happen outside of performing work or being labeled as complete
			UE_LOGF(LogAutomationControllerRpcBridge, Warning, "AutomationControllerManager ticking from an invalid state '%ls'.", *UEnum::GetValueAsString(AutomationControllerState));
		}

		// Clear delegate handles to avoid re-running tests due to multiple delegates being added or when refreshing session frontend
		// The handle will be registered in Init whenever a new command is executed
		ResetInitializationHandles();
		return false;
	}

	/**
	 * Generates a list of available test names.
	 *
	 * @param TestFilter			Filter to be parsed and checked against
	 * @param InFilters				Collection of automation filters
	 * @param OutFilteredTestNames	Resulting array of tests after all filters have been applied
	 */
	void GenerateTestNames(const FString& TestFilter, const TSharedPtr<AutomationFilterCollection>& InFilters, TArray<FString>& OutFilteredTestNames)
	{
		OutFilteredTestNames.Empty();

		// Split the argument names up on +
		TArray<FString> ArgumentNames;
		TestFilter.ParseIntoArray(ArgumentNames, TEXT("+"), true);

		// Get our settings CDO where the information is stored
		UAutomationControllerSettings* Settings = UAutomationControllerSettings::StaticClass()->GetDefaultObject<UAutomationControllerSettings>();

		// Iterate through the arguments to build a filter list by doing the following -
		//	1) If argument is a filter (StartsWith:system) then make sure we only filter-in tests that start with that filter
		//	2) If argument is a group then expand that group into multiple filters based on ini entries
		//	3) Otherwise just substring match
		TArray<FAutomatedTestFilter> FiltersList;
		for (int32 ArgumentIndex = 0; ArgumentIndex < ArgumentNames.Num(); ++ArgumentIndex)
		{
			const FString GroupPrefix = TEXT("Group:");
			const FString FilterPrefix = TEXT("StartsWith:");

			FString ArgumentName = ArgumentNames[ArgumentIndex].TrimStartAndEnd();

			// If the argument is a filter (e.g. Filter:System) then create a filter that matches from the start
			if (ArgumentName.StartsWith(FilterPrefix))
			{
				FString FilterName = ArgumentName.RightChop(FilterPrefix.Len()).TrimStart();

				if (FilterName.EndsWith(TEXT(".")) == false)
				{
					FilterName += TEXT(".");
				}

				FiltersList.Add(FAutomatedTestFilter(FilterName, true, false));
			}
			else if (ArgumentName.StartsWith(GroupPrefix))
			{
				// If the argument is a group (e.g. Group:Rendering) then search our groups for one that matches
				FString GroupName = ArgumentName.RightChop(GroupPrefix.Len()).TrimStart();

				bool FoundGroup = false;

				for (int32 i = 0; i < Settings->Groups.Num(); ++i)
				{
					FAutomatedTestGroup* GroupEntry = &(Settings->Groups[i]);
					if (GroupEntry && GroupEntry->Name == GroupName)
					{
						FoundGroup = true;
						// If found add all this groups filters to our current list
						if (GroupEntry->Filters.Num() > 0)
						{
							FiltersList.Append(GroupEntry->Filters);
						}
						else
						{
							UE_LOGF(LogAutomationControllerRpcBridge, Warning, "Group %ls contains no filters", *GroupName);
						}
					}
				}

				if (!FoundGroup)
				{
					UE_LOGF(LogAutomationControllerRpcBridge, Error, "No matching group named %ls", *GroupName);
				}
			}
			else
			{
				bool bMatchFromStart = false;
				bool bMatchFromEnd = false;

				if (ArgumentName.StartsWith("^"))
				{
					bMatchFromStart = true;
					ArgumentName.RightChopInline(1);
				}
				if (ArgumentName.EndsWith("$"))
				{
					bMatchFromEnd = true;
					ArgumentName.LeftChopInline(1);
				}

				FiltersList.Add(FAutomatedTestFilter(ArgumentName, bMatchFromStart, bMatchFromEnd));
			}
		}

		if (!FiltersList.IsEmpty())
		{
			TSharedPtr<FAutomationGroupFilter> FilterAny = MakeShareable(new FAutomationGroupFilter());
			FilterAny->SetFilters(FiltersList);
			InFilters->Add(FilterAny);

			// SetFilter applies all filters from the AutomationFilters array
			AutomationController->SetFilter(InFilters);
			// Fill OutFilteredTestNames array with filtered test names
			AutomationController->GetFilteredTestNames(OutFilteredTestNames);
		}
	}

	/** Simple method called from the Tick to update the status when tests have completed. */
	void MonitorTests()
	{
		if (AutomationController->GetTestState() != EAutomationControllerModuleState::Running)
		{
			// We have finished the testing, and results are available
			AutomationControllerState = EAutomationControllerState::Complete;
		}
	}

	/** The automation controller running the tests */
	IAutomationControllerManagerPtr AutomationController;

	/** The current state of the automation process */
	EAutomationControllerState AutomationControllerState = EAutomationControllerState::Uninitialized;

	/** Handle to Test Available delegate */
	FDelegateHandle TestsAvailableHandle;

	/** Handle to Test Report Generated delegate */
	FDelegateHandle TestReportGeneratedHandle;

	/** Handle to Test Refresh delegate */
	FDelegateHandle TestsRefreshedHandle;

	/** Holds the session ID */
	FGuid SessionID;

	//so we can release control of the app and just get ticked like all other systems
	FTSTicker::FDelegateHandle TickHandler;

	TSharedPtr<FJsonObject> GeneratedTestReport;
};

#endif //WITH_AUTOMATION_TESTS

UAutomationControllerRpcRegistrationComponent* UAutomationControllerRpcRegistrationComponent::ObjectInstance = nullptr;

UAutomationControllerRpcRegistrationComponent* UAutomationControllerRpcRegistrationComponent::GetInstance()
{
#if WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
	if (ObjectInstance == nullptr)
	{
		ObjectInstance = NewObject<UAutomationControllerRpcRegistrationComponent>();
		if (ObjectInstance == nullptr)
		{
			UE_LOGF(LogAutomationControllerRpcRegistrationComponent, Warning, "Unable to register the Rpc component.");
		}
		else
		{
			ObjectInstance->AddToRoot();
			ObjectInstance->RegisterAlwaysOnHttpCallbacks();
		}
	}
#endif // WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY

	return ObjectInstance;
}

void UAutomationControllerRpcRegistrationComponent::RegisterAlwaysOnHttpCallbacks()
{
	Super::RegisterAlwaysOnHttpCallbacks();

#if WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
	RegisterHttpCallback(FName(TEXT("AutomationControllerInitialize")),
		FHttpPath("/automation/controller/initialize"),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &ThisClass::HttpAutomationControllerInitializeCommand),
		true,
		TEXT("Automation"));

	RegisterHttpCallback(FName(TEXT("AutomationControllerGetState")),
		FHttpPath("/automation/controller/teststate"),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateUObject(this, &ThisClass::HttpAutomationControllerGetStateCommand),
		true,
		TEXT("Automation"));

	RegisterHttpCallback(FName(TEXT("AutomationControllerGetAvailableTests")),
		FHttpPath("/automation/controller/availabletests"),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateUObject(this, &ThisClass::HttpAutomationControllerGetAvailableTestsCommand),
		true,
		TEXT("Automation"));

	RegisterHttpCallback(FName(TEXT("AutomationControllerRunTests")),
		FHttpPath("/automation/controller/runtests"),
		EHttpServerRequestVerbs::VERB_POST,
		FHttpRequestHandler::CreateUObject(this, &ThisClass::HttpAutomationControllerRunTestsCommand),
		true,
		TEXT("Automation"));

	RegisterHttpCallback(FName(TEXT("AutomationControllerGenerateReports")),
		FHttpPath("/automation/controller/generatereports"),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateUObject(this, &ThisClass::HttpAutomationControllerGenerateReportsCommand),
		true,
		TEXT("Automation"));
#endif // WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
}

void UAutomationControllerRpcRegistrationComponent::DeregisterHttpCallbacks()
{
	Super::DeregisterHttpCallbacks();
}

void UAutomationControllerRpcRegistrationComponent::BeginDestroy()
{
	DeregisterHttpCallbacks();
	if (IsRooted())
	{
		RemoveFromRoot();
	}

	Super::BeginDestroy();
}

#if WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY

bool UAutomationControllerRpcRegistrationComponent::HttpAutomationControllerInitializeCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!AutomationControllerRpcBridge.IsValid())
	{
		AutomationControllerRpcBridge = MakeUnique<FAutomationControllerRpcBridge>();
	}

	FString ResponseStr = AutomationControllerRpcBridge->Init();

	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

bool UAutomationControllerRpcRegistrationComponent::HttpAutomationControllerGetStateCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!AutomationControllerRpcBridge.IsValid())
	{
		FString ResponseStr = CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not yet been initialized."));
		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
		OnComplete(MoveTemp(Response));
		return true;
	}

	FString Status;
	FString Description;
	EAutomationControllerState CurrentState = AutomationControllerRpcBridge->GetState();
	switch (CurrentState)
	{
		case EAutomationControllerState::Idle:
		case EAutomationControllerState::Complete:
		{
			Status = StatusIdle;
			Description = TEXT("AutomationController is ready to run tests.");
			break;
		}
		case EAutomationControllerState::FindingWorkers:
		case EAutomationControllerState::RequestTests:
		{
			Status = StatusInitializing;
			Description = TEXT("AutomationController is in the process of being setup.");
			break;
		}
		case EAutomationControllerState::DoingRequestedWork:
		{
			Status = StatusRunning;
			Description = TEXT("AutomationController is currently running tests.");
			break;
		}
		default:
		{
			Status = StatusError;
			Description = FString::Format(TEXT("AutomationController is in an invalid state '{0}'. Try initializing first."), { *UEnum::GetValueAsString(CurrentState) });
		}
	}
	
	FString ResponseStr;
	TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ResponseStr);
	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue("Status", Status);
	JsonWriter->WriteValue("Description", Description);
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

bool UAutomationControllerRpcRegistrationComponent::HttpAutomationControllerGetAvailableTestsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	if (!AutomationControllerRpcBridge.IsValid())
	{
		ResponseStr = CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not yet been initialized."));
		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
		OnComplete(MoveTemp(Response));
		return true;
	}

	const FString* TestFilterPtr = Request.QueryParams.Find("testfilter");
	FString TestFilter = TestFilterPtr != nullptr ? *TestFilterPtr : "";

	const FString* TagFilterPtr = Request.QueryParams.Find("tagfilter");
	FString TagFilter = TagFilterPtr != nullptr ? *TagFilterPtr : "";

	ResponseStr = AutomationControllerRpcBridge->GetAvailableTests(TestFilter, TagFilter);
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

bool UAutomationControllerRpcRegistrationComponent::HttpAutomationControllerRunTestsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	if (!AutomationControllerRpcBridge.IsValid())
	{
		ResponseStr = CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not yet been initialized."));
		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
		OnComplete(MoveTemp(Response));
		return true;
	}

	int32 TestLoops = 1;
	FString TestFilter;
	FString TagFilter;
	TSharedPtr<FJsonObject> BodyObject = GetJsonObjectFromRequestBody(Request.Body);
	if (BodyObject.IsValid())
	{
		BodyObject->TryGetNumberField(TEXT("TestLoops"), TestLoops);
		BodyObject->TryGetStringField(TEXT("TestFilter"), TestFilter);
		BodyObject->TryGetStringField(TEXT("TagFilter"), TagFilter);
	}

	ResponseStr = AutomationControllerRpcBridge->RunTests(TestFilter, TagFilter, TestLoops);
	
	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

bool UAutomationControllerRpcRegistrationComponent::HttpAutomationControllerGenerateReportsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	FString ResponseStr;
	if (!AutomationControllerRpcBridge.IsValid())
	{
		ResponseStr = CreateSimpleJsonReport(StatusError, TEXT("AutomationController has not yet been initialized."));
		TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
		OnComplete(MoveTemp(Response));
		return true;
	}

	ResponseStr = AutomationControllerRpcBridge->GenerateAutomationReportJson();

	TUniquePtr<FHttpServerResponse> Response = FHttpServerResponse::Create(ResponseStr, TEXT("application/json"));
	OnComplete(MoveTemp(Response));
	return true;
}

#endif // WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY