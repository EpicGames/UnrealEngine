// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class UAutomationControllerRpcRegistrationComponent;

class FAutomationControllerRpcModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:

	/** RPC registration component around the AutomationController */
	UAutomationControllerRpcRegistrationComponent* AutomationControllerRpcRegistrationComponent = nullptr;
};
