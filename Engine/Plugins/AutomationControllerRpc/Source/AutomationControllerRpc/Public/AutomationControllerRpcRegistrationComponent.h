// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AutomationTest.h"
#include "ExternalRpcRegistrationComponent.h"

#include "AutomationControllerRpcRegistrationComponent.generated.h"

#if WITH_AUTOMATION_TESTS
class FAutomationControllerRpcBridge;
#endif // WITH_AUTOMATION_TESTS

/** States for the underlying automation controller process */
UENUM()
enum class EAutomationControllerState : uint8
{
	Uninitialized,		// Automation process has not been setup
	Idle,				// Automation process is not running
	FindingWorkers,		// Find workers to run the tests
	RequestTests,		// Find the tests that can be run on the workers
	DoingRequestedWork,	// Do whatever was requested
	Complete			// The process is finished
};

UCLASS()
class AUTOMATIONCONTROLLERRPC_API UAutomationControllerRpcRegistrationComponent : public UExternalRpcRegistrationComponent
{
	GENERATED_BODY()

public:
	/** Gets the singleton instance of this component. */
	static UAutomationControllerRpcRegistrationComponent* GetInstance();

	/** Registers the necessary routes to the AutomationController. */
	void RegisterAlwaysOnHttpCallbacks() override;

	/** Deregisters all previously registered routes. */
	void DeregisterHttpCallbacks() override;

	void BeginDestroy() override;

private:

#if WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY
	/** Registers the route to initialize the AutomationController. */
	bool HttpAutomationControllerInitializeCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/** Registers the route to get the various states of the AutomationController. */
	bool HttpAutomationControllerGetStateCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/** Registers the route to fetch all registered tests from the AutomationController. */
	bool HttpAutomationControllerGetAvailableTestsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/** Registers the route to trigger test execution from the AutomationController. */
	bool HttpAutomationControllerRunTestsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	/** Registers the route to generate a report from the previous test execution from the AutomationController. */
	bool HttpAutomationControllerGenerateReportsCommand(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
#endif // WITH_AUTOMATION_TESTS && WITH_RPC_REGISTRY

	/** Instance of this RPC component. */
	static UAutomationControllerRpcRegistrationComponent* ObjectInstance;

#if WITH_AUTOMATION_TESTS
	/** Instance of the AutomationController. */
	TUniquePtr<FAutomationControllerRpcBridge> AutomationControllerRpcBridge = nullptr;
#endif // WITH_AUTOMATION_TESTS
};